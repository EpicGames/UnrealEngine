// Copyright Epic Games, Inc. All Rights Reserved.
// SPDX-License-Identifier: MIT
//
// Permission is hereby granted, free of charge, to any person obtaining a copy
// of this software and associated documentation files (the "Software"), to deal
// in the Software without restriction, including without limitation the rights
// to use, copy, modify, merge, publish, distribute, sublicense, and/or sell
// copies of the Software, and to permit persons to whom the Software is
// furnished to do so, subject to the following conditions:
// 
// The above copyright notice and this permission notice shall be included in all
// copies or substantial portions of the Software.
// 
// THE SOFTWARE IS PROVIDED "AS IS", WITHOUT WARRANTY OF ANY KIND, EXPRESS OR
// IMPLIED, INCLUDING BUT NOT LIMITED TO THE WARRANTIES OF MERCHANTABILITY,
// FITNESS FOR A PARTICULAR PURPOSE AND NONINFRINGEMENT. IN NO EVENT SHALL THE
// AUTHORS OR COPYRIGHT HOLDERS BE LIABLE FOR ANY CLAIM, DAMAGES OR OTHER
// LIABILITY, WHETHER IN AN ACTION OF CONTRACT, TORT OR OTHERWISE, ARISING FROM,
// OUT OF OR IN CONNECTION WITH THE SOFTWARE OR THE USE OR OTHER DEALINGS IN THE
// SOFTWARE.

#pragma once

#ifdef __cplusplus
#include <inttypes.h>
#include <stdint.h>
#include <errno.h>
#include <stdio.h>
#include <sys/socket.h>
#include <sys/un.h>
#include <unistd.h>

#define INVALID_FD (-1)

#ifndef PRINTF_ANSI_STR
#define PRINTF_ANSI_STR "%hs"
#endif

struct LinuxUEKernelToolsIPC
{
	static constexpr size_t MAX_EVENTS_PER_PACKET = 256; // maximum number of packets sent over the IPC socket in a single sendto()

	int Fd;

	LinuxUEKernelToolsIPC()
	: Fd(INVALID_FD)
	{
		Fd = socket(AF_UNIX, SOCK_DGRAM, 0);
		if (Fd == INVALID_FD)
		{
			UE_LOGF(LogLinuxUEKernelTools, Error, "Failed to open socket AF_UNIX, errno = %i (" PRINTF_ANSI_STR ")", errno, strerror(errno));
		}
	}

	~LinuxUEKernelToolsIPC()
	{
		if (Fd != INVALID_FD)
		{
			close(Fd);
			Fd = INVALID_FD;
		}
	}

	static constexpr size_t SocketBasePathMaxLen = sizeof(sockaddr_un::sun_path) - 10; // reserve 8 chars for pid, 1 for /, and 1 for null char
	static void FillSockAddrUn(sockaddr_un* Addr, const char* BasePath, pid_t Pid)
	{
		constexpr char NibbleToChar[16] = {'0', '1', '2', '3', '4', '5', '6', '7', '8', '9', 'a', 'b', 'c', 'd', 'e', 'f'};
		
		Addr->sun_family = AF_UNIX;
		Addr->sun_path[0] = '\0';
		strncpy(Addr->sun_path, BasePath, sizeof(Addr->sun_path) - 1);

		size_t BasePathLen = strlen(BasePath);
		if (BasePathLen > SocketBasePathMaxLen)
		{
			BasePathLen = SocketBasePathMaxLen;
		}

		Addr->sun_path[BasePathLen] = '/';
		char* Path = Addr->sun_path + BasePathLen + 1;
		Path[0] = NibbleToChar[Pid >> 28];
		Path[1] = NibbleToChar[(Pid >> 24) & 0xf];
		Path[2] = NibbleToChar[(Pid >> 20) & 0xf];
		Path[3] = NibbleToChar[(Pid >> 16) & 0xf];
		Path[4] = NibbleToChar[(Pid >> 12) & 0xf];
		Path[5] = NibbleToChar[(Pid >> 8) & 0xf];
		Path[6] = NibbleToChar[(Pid >> 4) & 0xf];
		Path[7] = NibbleToChar[Pid & 0xf];
		Path[8] = 0;
	}
};
#endif // #ifdef __cplusplus

typedef struct BPFEventCOWData
{
	uint64_t VirtualAddress;
#ifdef __cplusplus
	int32_t ToString(char* Buffer, size_t Len) const
	{
		return snprintf(Buffer, Len, "virt_ptr=0x%016" PRIx64, VirtualAddress);
	}
#endif
} BPFEventCOWData;

typedef enum EBPFEventType
{
	BPFEventIgnore = 0,
	BPFEventCOW,
} EBPFEventType;

#pragma pack(push, 1)
typedef struct BPFEvent
{
	uint32_t Pid;
	uint32_t Tid;
	uint16_t _Reserved1;
	uint8_t Type;
	uint8_t _Reserved2;
	union
	{
		BPFEventCOWData COW;
	};

#ifdef __cplusplus
	int32_t ToString(char* Buffer, size_t Len) const
	{
		int32_t Offset = snprintf(Buffer, Len, "pid=%016" PRIu32 " tid=%016" PRIu32 " type=", Pid, Tid);
		switch(Type)
		{
			case BPFEventCOW:
				Offset += snprintf(Buffer + Offset, Len - Offset, "cow ");
				Offset += COW.ToString(Buffer + Offset, Len - Offset);
				break;
				
			case BPFEventIgnore:
			default:
				Offset += snprintf(Buffer + Offset, Len - Offset, "ignored");
				break;
		}
		return Offset;
	}
#endif

} BPFEvent;
#pragma pack(pop)